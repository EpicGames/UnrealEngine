// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetRegistryExportCommandlet.h"

#include "AssetRegistry/AssetRegistryState.h"
#include "Containers/Queue.h"
#include "HAL/FileManager.h"
#include "Misc/App.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "Misc/StringBuilder.h"
#include "Modules/ModuleManager.h"
#include "Serialization/ArrayReader.h"
#include "Serialization/MemoryWriter.h"
#include "SQLiteDatabase.h"
#include "SQLitePreparedStatement.h"

DEFINE_LOG_CATEGORY(LogAssetRegistryExport);


UAssetRegistryExportCommandlet::UAssetRegistryExportCommandlet(const FObjectInitializer& Initializer)
	: Super(Initializer)
{
}

// Helper to comma separate numbers for readability.
static FString NumberString(uint64 N) { return FText::AsNumber(N).ToString(); }


int32 UAssetRegistryExportCommandlet::Main(const FString& CmdLineParams)
{
	FString OutputDatabaseName;
	FString AssetRegistryFileName;
	FString CSVFileName;
	FString PrimaryAssetTypes;

	FParse::Value(*CmdLineParams, TEXT("ListDependencies="), PrimaryAssetTypes);
	bool bIncludeSharedWithDependencies = FParse::Param(*CmdLineParams, TEXT("IncludeSharedWithDependencies"));

	// We must have either a sqlite or a csv output
	bool bHaveOutput = FParse::Value(*CmdLineParams, TEXT("Output="), OutputDatabaseName) ||
		FParse::Value(*CmdLineParams, TEXT("CSV="), CSVFileName);

	if (bHaveOutput == false ||
		FParse::Value(*CmdLineParams, TEXT("Input="), AssetRegistryFileName) == false)
	{
		UE_LOGF(LogAssetRegistryExport, Display, "");
		UE_LOGF(LogAssetRegistryExport, Display, "Asset Registry Exporter");
		UE_LOGF(LogAssetRegistryExport, Display, "");
		UE_LOGF(LogAssetRegistryExport, Display, "Converts a given asset registry to a sqlite/csv database. Each AssetClass gets its own");
		UE_LOGF(LogAssetRegistryExport, Display, "table with the corresponding tags for the class as columns. Assets are added to their");
		UE_LOGF(LogAssetRegistryExport, Display, "class's table, as well as a global Assets table.");
		UE_LOGF(LogAssetRegistryExport, Display, "");
		UE_LOGF(LogAssetRegistryExport, Display, "If the CompressedSize DB columns for all assets are NULL then the registry didn't have metadata ");
		UE_LOGF(LogAssetRegistryExport, Display, "written back (ProjectsSettings/Packaging/WriteBackMetadataToAssetRegistry)");
		UE_LOGF(LogAssetRegistryExport, Display, "");
		UE_LOGF(LogAssetRegistryExport, Display, "Parameters:");
		UE_LOGF(LogAssetRegistryExport, Display, "");
		UE_LOGF(LogAssetRegistryExport, Display, "    -Output=<path/to/file>              Output Sqlite DB file (Required Or CSV)");
		UE_LOGF(LogAssetRegistryExport, Display, "    -CSV=<path>                         Directory for CSV output files (Required Or Sqlite) MAKES A TON OF FILES.");
		UE_LOGF(LogAssetRegistryExport, Display, "    -Input=<path/to/file>               Input Asset Registry file (Required)");
		UE_LOGF(LogAssetRegistryExport, Display, "    -ListDependencies=typea,typeb       Comma separated list of primary asset types to output dependency");
		UE_LOGF(LogAssetRegistryExport, Display, "                                        information. Requires asset registry metadata and to be running as");
		UE_LOGF(LogAssetRegistryExport, Display, "                                        a commandlet for the project that generated the asset registry.");
		UE_LOGF(LogAssetRegistryExport, Display, "    -IncludeSharedWithDependencies      When listing dependencies, this will also add shared dependencies");
		UE_LOGF(LogAssetRegistryExport, Display, "                                        to a \"shared\" root.");
		UE_LOGF(LogAssetRegistryExport, Display, "    -FilterToClass=<classname>          Only add the table for the given class (and global assets)");
		UE_LOGF(LogAssetRegistryExport, Display, "                                        Example: -FilterToClass=\"/Script/Engine.Texture2D\"");
		return 1;
	}

	FSQLiteDatabase OutputDatabase;

	bool bCSV = CSVFileName.Len() != 0;

	if (bCSV == false)
	{
		// We get a mountain of errors if we append to a database.
		if (!IFileManager::Get().Delete(*OutputDatabaseName, false))
		{
			UE_LOGF(LogAssetRegistryExport, Error, "Failed to delete sqlite database before opening %ls", *OutputDatabaseName);
			return 1;
		}
		if (OutputDatabase.Open(*OutputDatabaseName) == false)
		{
			UE_LOGF(LogAssetRegistryExport, Error, "Failed to open sqlite database %ls", *OutputDatabaseName);
			return 1;
		}
		UE_LOGF(LogAssetRegistryExport, Display, "Creating Sqlite DB: %ls", *OutputDatabaseName);
	}
	else
	{
		FPaths::MakePlatformFilename(CSVFileName);
		if (CSVFileName.EndsWith(FPlatformMisc::GetDefaultPathSeparator()) == false)
		{
			CSVFileName += FPlatformMisc::GetDefaultPathSeparator();
		}
	}

	TSet<FName> DependencyListAssetTypeNames;
	if (PrimaryAssetTypes.Len())
	{
		TArray<FString> DependencyListAssetTypes;
		PrimaryAssetTypes.ParseIntoArray(DependencyListAssetTypes, TEXT(","), true);
		for (FString& AssetType : DependencyListAssetTypes)
		{
			DependencyListAssetTypeNames.Add(FName(AssetType));
		}
	}

	FAssetRegistryState AssetRegistry;
	{
		FArrayReader SerializedAssetData;
		if (FFileHelper::LoadFileToArray(SerializedAssetData, *AssetRegistryFileName))
		{
			FAssetRegistrySerializationOptions Options(UE::AssetRegistry::ESerializationTarget::ForDevelopment);
			if (AssetRegistry.Serialize(SerializedAssetData, Options) == false)
			{
				UE_LOGF(LogAssetRegistryExport, Error, "Failed to serialize asset registry %ls", *AssetRegistryFileName);
				return 1;
			}
		}
		else
		{
			UE_LOGF(LogAssetRegistryExport, Error, "Failed to read asset registry %ls", *AssetRegistryFileName);
			return 1;
		}
	}
	UE_LOGF(LogAssetRegistryExport, Display, "Opened asset registry: %ls", *AssetRegistryFileName);

	if (DependencyListAssetTypeNames.Num())
	{
		// We need the project loaded to do the dependency listing because we look at the class hierarchy
		// which currently isn't stored in the asset registry.
		FString CheckForProjectName = FString("/Script/") + FApp::GetProjectName();
		TArray<FAssetDependency> CheckDeps;
		AssetRegistry.GetReferencers(FAssetIdentifier(*CheckForProjectName), CheckDeps);
		if (CheckDeps.Num() == 0)
		{
			UE_LOGF(LogAssetRegistryExport, Error, "Can't list dependencies without running as a commandlet for the same project that generated the DevelopmentAssetRegistry");
			return 1;
		}
	}

	// Set up some database properties for speed - we don't really care about ACID in this case
	// since we're just dumping stuff and closing the database - we can always start over.
	if (bCSV == false)
	{
		// don't bother waiting for disk to flush (let the OS handle it)
		OutputDatabase.Execute(TEXT("PRAGMA synchronous=OFF;"));

		// 1000 page cache
		OutputDatabase.Execute(TEXT("PRAGMA cache_size=1000;"));

		// large-ish page sizes for modern disks
		OutputDatabase.Execute(TEXT("PRAGMA page_size=65535;"));

		// hold the file lock the whole time.
		OutputDatabase.Execute(TEXT("PRAGMA locking_mode=EXCLUSIVE;"));
	}

	FString FilterToClass;
	FParse::Value(*CmdLineParams, TEXT("FilterToClass="), FilterToClass);
	if (FilterToClass.Len())
	{
		UE_LOGF(LogAssetRegistryExport, Display, "Got option: FilterToClass=%ls", *FilterToClass);
	}

	struct FClassInfo
	{
		FString Name;
		TSet<FName> Tags;

		// TagName -> Name that has been sanitized for use as a column name.
		TMap<FName, FString> SanitizedTagNames;

		// TagName -> index for binding in the insert statement.
		TMap<FName, int32> TagBindingIndices;

		FSQLitePreparedStatement InsertStatement;

		TArray64<uint8> CSVMemory;
	};

	TMap<FTopLevelAssetPath, FClassInfo> ClassInfos;
	int32 AssetCount = 0;

	// Find the tags for each class. Assets might not fill all tags that might exist on a
	// given class, so we have to get the union of all possibles.
	AssetRegistry.EnumerateAllAssets(TSet<FName>(), [&](const FAssetData& AssetData)
	{
		FString AssetClassPathString = AssetData.AssetClassPath.ToString();

		if (FilterToClass.Len() &&
			FilterToClass != AssetClassPathString)
		{
			return true;
		}

		FClassInfo& ClassInfo = ClassInfos.FindOrAdd(AssetData.AssetClassPath);
		ClassInfo.Name = MoveTemp(AssetClassPathString);
		{
			auto& CharArray = ClassInfo.Name.GetCharArray();
			for (int i = 0; i < CharArray.Num() - 1; i++)
			{
				if (TChar<TCHAR>::IsAlnum(CharArray[i]) == false)
				{
					CharArray[i] = TEXT('_');
				}
			}
		}

		AssetData.TagsAndValues.ForEach([&ClassInfo](TPair<FName, FAssetTagValueRef> Pair)
		{
			bool bAlreadySet = false;
			ClassInfo.Tags.Add(Pair.Key, &bAlreadySet);

			if (bAlreadySet == false)
			{
				FString& SanitizedTagName = ClassInfo.SanitizedTagNames.FindOrAdd(Pair.Key, Pair.Key.ToString());

				// Only alphanumeric and _ are allowed, just make anything else a "_".
				// Num() - 1 because FStrings store the null term in the array
				auto& CharArray = SanitizedTagName.GetCharArray();
				for (int i = 0; i < CharArray.Num() - 1; i++)
				{
					if (TChar<TCHAR>::IsAlnum(CharArray[i]) == false)
					{
						CharArray[i] = TEXT('_');
					}
				}
			}
		});

		AssetCount++;
		return true;
	});

	if (DependencyListAssetTypeNames.Num())
	{
		// function to try to find a base class that makes sense for the given asset.
		auto GetRootClassToPrint = [](FTopLevelAssetPath AssetClassPath)
		{
			UClass* AssetClass = FindObject<UClass>(AssetClassPath, EFindObjectFlags::ExactClass);
			if (AssetClass == nullptr)
			{
				return FString("Unknown");
			}

			UClass* OriginalClass = AssetClass;
			FString ClassName;
			while (UClass* Super = AssetClass->GetSuperClass())
			{				
				Super->GetName(ClassName);

				// Stop when we get to something too generic.
				if (ClassName == "Object" || ClassName == "BlueprintCore" || ClassName == "DataAsset" || ClassName == "StreamableRenderAsset" || ClassName == "SoundBase")
				{
					break;
				}
				AssetClass = Super;
			}

			if (AssetClass)
			{
				return AssetClass->GetName();
			}
			else
			{
				return OriginalClass->GetName();
			}
		};

		//
		// Create a table to hold each game feature data dependency list.
		//

		// Find all the assets that are of the requested primary asset types. These we'll use as roots for the 
		// dependency listing.
		TArray<const FAssetData*> PrimaryAssets;
		AssetRegistry.EnumerateAssetsByTagName(FPrimaryAssetId::PrimaryAssetTypeTag, 
			[&DependencyListAssetTypeNames, &PrimaryAssets](const FAssetData* GFD)
			{
				FName PrimaryAssetType;
				if (GFD->GetTagValue(FPrimaryAssetId::PrimaryAssetTypeTag, PrimaryAssetType) &&
					DependencyListAssetTypeNames.Contains(PrimaryAssetType))
				{
					PrimaryAssets.Add(GFD);
				}

				return true;
			});

		
		UE_LOGF(LogAssetRegistryExport, Display, "ListDependencies matching PrimaryAssets discovered %ls", *NumberString(PrimaryAssets.Num()));

		// Track all assets with size specified so we can know who isn't assigned.
		TSet<const FAssetData*> RemainingAssetsWithSize;
		uint64 TotalCompressedBytes = 0;
		{
			AssetRegistry.EnumerateAssetsByTagName(UE::AssetRegistry::Stage_ChunkCompressedSizeFName,
				[&RemainingAssetsWithSize, &TotalCompressedBytes](const FAssetData* AD)
				{
					uint64 CompressedSize = 0;
					if (AD->GetTagValue(UE::AssetRegistry::Stage_ChunkCompressedSizeFName, CompressedSize))
					{
						TotalCompressedBytes += CompressedSize;
						RemainingAssetsWithSize.Add(AD);
					}

					return true;
				});
		}

		const TMap<FName, const FAssetPackageData*>& AllPackages = AssetRegistry.GetAssetPackageDataMap();

		TSet<FName> UnionedTotalDependencies;
		TSet<FName> UnionUniqueDependencies;

		struct FPrimaryAssetInfo
		{
			TSet<FName> TotalPackageDependencies;
			TSet<FName> UniquePackageDependencies;

			uint64 SizeOfAllDependencies = 0;
			uint64 SizeOfUniqueDependencies = 0;
		};
	
		TMap<FName, FPrimaryAssetInfo> PrimaryAssetInfos;

		uint64 TotalSizesOfUniqueDependencies = 0;
		uint64 UnassignedSize = 0;
		uint64 SharedSize = 0;

		{
			TArray64<uint8> CSV;
			FMemoryWriter64 CSVWriter(CSV);

			{
				TUtf8StringBuilder<32> Header;
				Header << "RootPrimaryAsset,Name,Class,RootClass,Stage_ChunkCompressedSize";
				Header << "\r\n";
				CSVWriter.Serialize(Header.GetData(), Header.Len());
			}

			for (const FAssetData* PrimaryAsset : PrimaryAssets)
			{
				FPrimaryAssetInfo& Info = PrimaryAssetInfos.Add(PrimaryAsset->PackageName);
				Info.TotalPackageDependencies.Add(PrimaryAsset->PackageName);

				// Get recursive dependencies
				{
					TQueue<FName> PackageQueue;
					PackageQueue.Enqueue(PrimaryAsset->PackageName);

					FName PackageName;
					while (PackageQueue.Dequeue(PackageName))
					{
						TArray<const FAssetData*> AssetDataList = AssetRegistry.CopyAssetsByPackageName(PackageName);

						TArray<FAssetIdentifier> PackageDependencies;
						AssetRegistry.GetDependencies(FAssetIdentifier(PackageName), PackageDependencies);

						for (FAssetIdentifier PackageDependency : PackageDependencies)
						{
							if (PackageDependency.IsPackage())
							{
								bool bAlreadyExisted = false;
								Info.TotalPackageDependencies.Add(PackageDependency.PackageName, &bAlreadyExisted);

								if (bAlreadyExisted == false)
								{
									PackageQueue.Enqueue(PackageDependency.PackageName);
								}
							}
						}
					}
				}

				

				// Write out the total dependencies and sum the size of everything. (this is the slow part)
				Info.SizeOfAllDependencies = 0;
				TUtf8StringBuilder<2048> Line;
				for (FName& PackageDependency : Info.TotalPackageDependencies)
				{
					Line.Reset();
					Line << PrimaryAsset->GetObjectPathString() << ",";
					Line << PackageDependency.ToString() << ",";

					const FAssetData* UseAsClassData = UE::AssetRegistry::GetMostImportantAsset(
						AssetRegistry.CopyAssetsByPackageName(PackageDependency),
						UE::AssetRegistry::EGetMostImportantAssetFlags::IgnoreSkipClasses);
					if (UseAsClassData)
					{
						Line << UseAsClassData->AssetClassPath.ToString() << ",";
						Line << GetRootClassToPrint(UseAsClassData->AssetClassPath) << ",";

						uint64 CompressedSize = 0;
						if (UseAsClassData->GetTagValue(UE::AssetRegistry::Stage_ChunkCompressedSizeFName, CompressedSize))
						{
							Info.SizeOfAllDependencies += CompressedSize;
						}
						Line << CompressedSize << ",";
					}
					else
					{
						Line << "Unknown,,";
					}
			
					Line << "\r\n";
					CSVWriter.Serialize(Line.GetData(), Line.Len());

					UnionedTotalDependencies.Add(PackageDependency);
				}

				

			} // end for each root


			TMap<const FAssetData*, FName> AccountedFor;

			// Now figure out what's unique to each root. Iterate over all dependencies for
			// all roots and mark when we see it more than once.
			TMap<FName, bool> IsDependencyUnique;
			for (TPair<FName, FPrimaryAssetInfo>& Pair : PrimaryAssetInfos)
			{
				for (FName PackageDependency : Pair.Value.TotalPackageDependencies)
				{
					bool* bIsUnique = IsDependencyUnique.Find(PackageDependency);
					if (bIsUnique)
					{
						*bIsUnique = false; // we've seen it before so it can't be unique.
					}
					else
					{
						IsDependencyUnique.Add(PackageDependency, true);
					}
				}
			}

			// Now that we have uniqueness, go back and copy the entries for each root that
			// are unique and sum the size.
			for (TPair<FName, FPrimaryAssetInfo>& Pair : PrimaryAssetInfos)
			{
				for (FName PackageDependency : Pair.Value.TotalPackageDependencies)
				{
					bool bIsUnique = IsDependencyUnique.FindChecked(PackageDependency);
					if (bIsUnique)
					{
						Pair.Value.UniquePackageDependencies.Add(PackageDependency);
						UnionUniqueDependencies.Add(PackageDependency);

						const FAssetData* UseAsClassData = UE::AssetRegistry::GetMostImportantAsset(
							AssetRegistry.CopyAssetsByPackageName(PackageDependency),
							UE::AssetRegistry::EGetMostImportantAssetFlags::IgnoreSkipClasses);
						if (UseAsClassData)
						{
							uint64 CompressedSize = 0;
							if (UseAsClassData->GetTagValue(UE::AssetRegistry::Stage_ChunkCompressedSizeFName, CompressedSize))
							{
								check(RemainingAssetsWithSize.Contains(UseAsClassData));
								RemainingAssetsWithSize.Remove(UseAsClassData);

								if (AccountedFor.Contains(UseAsClassData))
								{
									UE_LOGF(LogAssetRegistryExport, Warning, "Already seen %ls in %ls, we are %ls", *PackageDependency.ToString(), *AccountedFor[UseAsClassData].ToString(), *Pair.Key.ToString());
								}
								AccountedFor.Add(UseAsClassData, Pair.Key);

								Pair.Value.SizeOfUniqueDependencies += CompressedSize;
							}
						}
					}
				}

				TotalSizesOfUniqueDependencies += Pair.Value.SizeOfUniqueDependencies;
			}

			for (const TPair<FName, const FAssetPackageData*>& Package : AllPackages)
			{			
				if (UnionedTotalDependencies.Contains(Package.Key) == false)
				{
					// If the package isn't in _any_ dependency, it's unassigned.
					TUtf8StringBuilder<128> Line;
					Line << "Unassigned,";
					Line << Package.Key.ToString() << ",";

					const FAssetData* UseAsClassData = UE::AssetRegistry::GetMostImportantAsset(
						AssetRegistry.CopyAssetsByPackageName(Package.Key),
						UE::AssetRegistry::EGetMostImportantAssetFlags::IgnoreSkipClasses);
					if (UseAsClassData)
					{
						Line << UseAsClassData->AssetClassPath.ToString() << ",";
						Line << GetRootClassToPrint(UseAsClassData->AssetClassPath) << ",";

						uint64 CompressedSize = 0;
						if (UseAsClassData->GetTagValue(UE::AssetRegistry::Stage_ChunkCompressedSizeFName, CompressedSize))
						{
							check(RemainingAssetsWithSize.Contains(UseAsClassData));
							RemainingAssetsWithSize.Remove(UseAsClassData);

							if (AccountedFor.Contains(UseAsClassData))
							{
								UE_LOGF(LogAssetRegistryExport, Warning, "Already seen %ls in %ls, we are %ls", *Package.Key.ToString(), *AccountedFor[UseAsClassData].ToString(), *Package.Key.ToString());
							}
							AccountedFor.Add(UseAsClassData, Package.Key);

							UnassignedSize += CompressedSize;
							RemainingAssetsWithSize.Remove(UseAsClassData);
							Line << CompressedSize;
						}					
					}
					else
					{
						Line << "Unknown,";
					}

					Line << "\r\n";
					CSVWriter.Serialize(Line.GetData(), Line.Len());
					continue;
				}

				if (UnionUniqueDependencies.Contains(Package.Key) == false)
				{
					// If the package IS _any_ dependency, but not in any _unique_ dependency, it's a shared
					// dependency.
					TUtf8StringBuilder<128> Line;
					Line << "Shared,";
					Line << Package.Key.ToString() << ",";

					const FAssetData* UseAsClassData = UE::AssetRegistry::GetMostImportantAsset(
						AssetRegistry.CopyAssetsByPackageName(Package.Key),
						UE::AssetRegistry::EGetMostImportantAssetFlags::IgnoreSkipClasses);
					if (UseAsClassData)
					{
						Line << UseAsClassData->AssetClassPath.ToString() << ",";
						Line << GetRootClassToPrint(UseAsClassData->AssetClassPath) << ",";

						uint64 CompressedSize = 0;
						if (UseAsClassData->GetTagValue(UE::AssetRegistry::Stage_ChunkCompressedSizeFName, CompressedSize))
						{
							check(RemainingAssetsWithSize.Contains(UseAsClassData));
							RemainingAssetsWithSize.Remove(UseAsClassData);

							if (AccountedFor.Contains(UseAsClassData))
							{
								UE_LOGF(LogAssetRegistryExport, Warning, "Already seen %ls in %ls, we are %ls", *Package.Key.ToString(), *AccountedFor[UseAsClassData].ToString(), *Package.Key.ToString());
							}
							AccountedFor.Add(UseAsClassData, Package.Key);

							SharedSize += CompressedSize;
							RemainingAssetsWithSize.Remove(UseAsClassData);
							Line << CompressedSize;
						}
					}
					else
					{
						Line << "Unknown,";
					}

					Line << "\r\n";

					if (bIncludeSharedWithDependencies)
					{
						CSVWriter.Serialize(Line.GetData(), Line.Len());
					}
				}
			}

			{
				FArchive* Ar = IFileManager::Get().CreateFileWriter(*(CSVFileName + TEXT("ListedDependencies.csv")));
				Ar->Serialize(CSV.GetData(), CSV.Num());
				delete Ar;
			}

			// Write out a shorter CSV that just summarizes the size of the various roots.
			{
				TArray64<uint8> SizeCSV;
				FMemoryWriter64 SizeCSVWriter(SizeCSV);

				{
					TUtf8StringBuilder<32> Header;
					Header << "RootPrimaryAsset,AllDependenciesCompressedBytes,UniqueDependenciesCompressedBytes";
					Header << "\r\n";
					SizeCSVWriter.Serialize(Header.GetData(), Header.Len());
				}

				for (TPair<FName, FPrimaryAssetInfo>& Pair : PrimaryAssetInfos)
				{
					TUtf8StringBuilder<128> Line;

					Line << Pair.Key.ToString() << ",";
					Line << Pair.Value.SizeOfAllDependencies << ",";
					Line << Pair.Value.SizeOfUniqueDependencies;
					Line << "\r\n";
					SizeCSVWriter.Serialize(Line.GetData(), Line.Len());
				}

				{
					TUtf8StringBuilder<128> Line;

					Line <<"Unassigned,";
					Line << UnassignedSize << ",";
					Line << UnassignedSize;
					Line << "\r\n";
					SizeCSVWriter.Serialize(Line.GetData(), Line.Len());
				}

				{
					TUtf8StringBuilder<128> Line;

					Line << "Shared,";
					Line << SharedSize << ",";
					Line << SharedSize;
					Line << "\r\n";
					SizeCSVWriter.Serialize(Line.GetData(), Line.Len());
				}

				FArchive* Ar = IFileManager::Get().CreateFileWriter(*(CSVFileName + TEXT("SizeDependencies.csv")));
				Ar->Serialize(SizeCSV.GetData(), SizeCSV.Num());
				delete Ar;
			}
		}

		UE_LOGF(LogAssetRegistryExport, Display, "TotalUnique: %ls, Shared: %ls Unassigned: %ls, Total %ls, Orphaned (%ls assets) %ls",
			*NumberString(TotalSizesOfUniqueDependencies),
			*NumberString(SharedSize),
			*NumberString(UnassignedSize), 
			*NumberString(TotalCompressedBytes), 
			*NumberString(RemainingAssetsWithSize.Num()),
			*NumberString(TotalCompressedBytes - UnassignedSize - TotalSizesOfUniqueDependencies - SharedSize));

		if (RemainingAssetsWithSize.Num())
		{
			UE_LOGF(LogAssetRegistryExport, Display, "%ls orphaned assets found - this is due to differences in assigning the \"most important\" asset in a package between UnrealPak and here.", *NumberString(RemainingAssetsWithSize.Num()));
		}
		for (const FAssetData* LD : RemainingAssetsWithSize)
		{
			uint64 value = 0;
			if (LD->GetTagValue(UE::AssetRegistry::Stage_ChunkCompressedSizeFName, value))
			{
				const FAssetPackageData* PackageData = AllPackages[LD->PackageName];

				UE_LOGF(LogAssetRegistryExport, Warning, "...Orphaned: %ls (%ls bytes, in packagemap: %ls", *LD->PackageName.ToString(), *NumberString(value), *LexToString(AllPackages.Contains(LD->PackageName)));

				TArray<const FAssetData*> PackageAssets = AssetRegistry.CopyAssetsByPackageName(LD->PackageName);
				for (const FAssetData* AD : PackageAssets)
				{
					uint64 blah = 0;
					AD->GetTagValue(UE::AssetRegistry::Stage_ChunkCompressedSizeFName, blah);

					UE_LOGF(LogAssetRegistryExport, Warning, "......Package has %ls %lld", *AD->AssetName.ToString(), blah);

				}

				const FAssetData* MostImportant = UE::AssetRegistry::GetMostImportantAsset(PackageAssets, UE::AssetRegistry::EGetMostImportantAssetFlags::IgnoreSkipClasses);
				UE_LOGF(LogAssetRegistryExport, Warning, "......MostImportant: %ls", MostImportant ? *MostImportant->AssetName.ToString() : TEXT("null"));
			}
		}
	} // end if dependency list is requested.

	//
	// Create the tables and insert statements for each class
	//
	for (TPair<FTopLevelAssetPath, FClassInfo>& Pair : ClassInfos)
	{
		FClassInfo& Class = Pair.Value;

		if (bCSV)
		{
			TUtf8StringBuilder<128> Header;
			Header << "Name,Class,Path";

			for (FName TagName : Class.Tags)
			{
				FString& SanitizedTagName = Class.SanitizedTagNames[TagName];
				Header << "," << SanitizedTagName;
			}
			Header << "\r\n";

			FMemoryWriter64 CSVWriter(Class.CSVMemory);
			CSVWriter.Serialize(Header.GetData(), Header.Len());
		}
		else
		{
			FString CreateTable = TEXT("CREATE TABLE IF NOT EXISTS ") + Class.Name + TEXT("Assets(Id INTEGER PRIMARY KEY, Name, Class, Path TEXT UNIQUE");
			FString InsertStatement = TEXT("INSERT INTO ") + Class.Name + TEXT("Assets(Name, Class, Path");
			FString ValuesStatement = TEXT("Values(?1, ?2, ?3");
			int32 CurrentIndex = 4;

			for (FName TagName : Class.Tags)
			{
				FString& SanitizedTagName = Class.SanitizedTagNames[TagName];

				CreateTable += TEXT(", \"") + SanitizedTagName + TEXT("\"");
				InsertStatement += TEXT(", \"") + SanitizedTagName + TEXT("\"");
				ValuesStatement += FString::Printf(TEXT(", ?%d"), CurrentIndex);

				Class.TagBindingIndices.Add(TagName, CurrentIndex);
				CurrentIndex++;
			}
			CreateTable += TEXT(");");
			InsertStatement += TEXT(") ") + ValuesStatement + TEXT(");");

			if (OutputDatabase.Execute(*CreateTable) == false)
			{
				UE_LOGF(LogAssetRegistryExport, Error, "Table Create Error: %ls", *OutputDatabase.GetLastError());
				OutputDatabase.Close();
				return 1;
			}

			if (Class.InsertStatement.Create(OutputDatabase, *InsertStatement) == false)
			{
				UE_LOGF(LogAssetRegistryExport, Error, "Insert Statement Create Error: %ls", *OutputDatabase.GetLastError());
				OutputDatabase.Close();
				return 1;
			}
		}
	}

	TArray64<uint8> AssetsCSV;
	TOptional<FSQLitePreparedStatement> GlobalInsertStatement;
	TOptional<FSQLitePreparedStatement> BeginTransactionStatement;
	TOptional<FSQLitePreparedStatement> CommitTransactionStatement;
	if (bCSV)
	{
		FMemoryWriter64 CSVWriter(AssetsCSV);
		
		TUtf8StringBuilder<128> Header;
		Header << "Name,Class,Path,Stage_ChunkCompressedSize,Stage_ChunkSize,Stage_ChunkOptionalSize,Stage_ChunkStreamingSize\r\n";
		CSVWriter.Serialize(Header.GetData(), Header.Len());
	}
	else
	{
		// Create catchall table and insert statements
		if (OutputDatabase.Execute(TEXT("CREATE TABLE IF NOT EXISTS Assets(Id INTEGER PRIMARY KEY, Name, Class, Path TEXT UNIQUE, Stage_ChunkCompressedSize INTEGER, Stage_ChunkSize INTEGER, Stage_ChunkOptionalSize INTEGER, Stage_ChunkStreamingSize INTEGER);")) == false)
		{
			UE_LOGF(LogAssetRegistryExport, Error, "Error: %ls", *OutputDatabase.GetLastError());
			OutputDatabase.Close();
			return 1;
		}

		GlobalInsertStatement.Emplace(OutputDatabase, TEXT("INSERT INTO Assets(Name, Class, Path, Stage_ChunkCompressedSize, Stage_ChunkSize, Stage_ChunkOptionalSize, Stage_ChunkStreamingSize) Values(?1, ?2, ?3, ?4, ?5, ?6, ?7);"));
		if (GlobalInsertStatement.GetPtrOrNull()->IsValid() == false)
		{
			UE_LOGF(LogAssetRegistryExport, Error, "Error: %ls", *OutputDatabase.GetLastError());
			OutputDatabase.Close();
			return 1;
		}

		BeginTransactionStatement.Emplace(OutputDatabase, TEXT("BEGIN TRANSACTION;"));
		CommitTransactionStatement.Emplace(OutputDatabase, TEXT("COMMIT TRANSACTION;"));
	}

	UE_LOGF(LogAssetRegistryExport, Display, "Adding %d assets to the database...", AssetCount);

	double StartTime = FPlatformTime::Seconds();

	// It's probably fine to add everything in one transaction, however add
	// a limit for sanity.
	const uint32 AssetsPerTransaction = 100000;
	uint32 AssetsThisTransaction = 0;

	if (bCSV == false)
	{
		BeginTransactionStatement.GetPtrOrNull()->Execute();
	}
	
	// Actually add the assets.
	bool bGotCompressedSize = false;
	AssetRegistry.EnumerateAllAssets(TSet<FName>(), [&](const FAssetData& AssetData)
	{
		FString AssetObjectPath = AssetData.GetObjectPathString();

		if (bCSV)
		{
			// Add to the 'Assets' table.
			{
				FMemoryWriter64 CSVWriter(AssetsCSV, true, true);

				TUtf8StringBuilder<128> Line;
				Line << AssetData.AssetName.ToString() << TEXTVIEW(",");
				Line << AssetData.AssetClassPath.ToString() << TEXTVIEW(",");
				Line << AssetObjectPath << TEXTVIEW(",");
				
				// If this assets has size information then add it to the global assets list as well.
				FString CompressedSize;
				if (AssetData.GetTagValue(UE::AssetRegistry::Stage_ChunkCompressedSizeFName, CompressedSize))
				{
					bGotCompressedSize = true;
				}
				FString Size;
				AssetData.GetTagValue(UE::AssetRegistry::Stage_ChunkSizeFName, Size);
				FString StreamingSize;
				AssetData.GetTagValue(UE::AssetRegistry::Stage_ChunkStreamingSizeFName, StreamingSize);
				FString OptionalSize;
				AssetData.GetTagValue(UE::AssetRegistry::Stage_ChunkOptionalSizeFName, OptionalSize);
				
				Line << CompressedSize << TEXTVIEW(",");
				Line << Size << TEXTVIEW(",");
				Line << OptionalSize << TEXTVIEW(",");
				Line << StreamingSize;

				Line << TEXTVIEW("\r\n");

				CSVWriter.Serialize(Line.GetData(), Line.Len());
			}

			// Add to our class's table.
			FClassInfo* ClassPtr = ClassInfos.Find(AssetData.AssetClassPath);
			if (ClassPtr)
			{
				FClassInfo& Class = *ClassPtr;

				FMemoryWriter64 CSVWriter(Class.CSVMemory, true, true);

				TUtf8StringBuilder<128> Line;
				Line << AssetData.AssetName.ToString() << ",";
				Line << AssetData.AssetClassPath.ToString() << ",";
				Line << AssetObjectPath;

				for (FName TagName : Class.Tags)
				{
					FString Value;
					AssetData.GetTagValue(TagName, Value);
					Line << "," << Value;
				}

				Line << "\r\n";

				CSVWriter.Serialize(Line.GetData(), Line.Len());
			}

		}
		else
		{
			// Add to the 'Assets' table.
			{
				GlobalInsertStatement.GetPtrOrNull()->ClearBindings();

				GlobalInsertStatement.GetPtrOrNull()->SetBindingValueByIndex(1, AssetData.AssetName.ToString());
				GlobalInsertStatement.GetPtrOrNull()->SetBindingValueByIndex(2, AssetData.AssetClassPath.ToString());
				GlobalInsertStatement.GetPtrOrNull()->SetBindingValueByIndex(3, AssetObjectPath);

				// If this assets has size information then add it to the global assets list as well.
				FString CompressedSize;
				if (AssetData.GetTagValue(UE::AssetRegistry::Stage_ChunkCompressedSizeFName, CompressedSize))
				{
					bGotCompressedSize = true;
					GlobalInsertStatement.GetPtrOrNull()->SetBindingValueByIndex(4, CompressedSize);
				} 

				FString Size;
				if (AssetData.GetTagValue(UE::AssetRegistry::Stage_ChunkSizeFName, Size))
				{
					GlobalInsertStatement.GetPtrOrNull()->SetBindingValueByIndex(5, Size);
				}

				FString StreamingSize;
				if (AssetData.GetTagValue(UE::AssetRegistry::Stage_ChunkStreamingSizeFName, StreamingSize))
				{
					GlobalInsertStatement.GetPtrOrNull()->SetBindingValueByIndex(6, StreamingSize);
				}
				FString OptionalSize;
				if (AssetData.GetTagValue(UE::AssetRegistry::Stage_ChunkOptionalSizeFName, OptionalSize))
				{
					GlobalInsertStatement.GetPtrOrNull()->SetBindingValueByIndex(7, OptionalSize);
				}

				if (GlobalInsertStatement.GetPtrOrNull()->Execute() == false)
				{
					UE_LOGF(LogAssetRegistryExport, Error, "Insertion Error: %ls", *OutputDatabase.GetLastError());
				}
			}
		

			// Add to our class's table.
			FClassInfo* ClassPtr = ClassInfos.Find(AssetData.AssetClassPath);
			if (ClassPtr)
			{
				FClassInfo& Class = *ClassPtr;

				Class.InsertStatement.ClearBindings();

				Class.InsertStatement.SetBindingValueByIndex(1, AssetData.AssetName.ToString());
				Class.InsertStatement.SetBindingValueByIndex(2, AssetData.AssetClassPath.ToString());
				Class.InsertStatement.SetBindingValueByIndex(3, AssetObjectPath);

				// This will leave any tag values unset (NULL) in the insertion, which is what we want.
				AssetData.EnumerateTags([&Class, &AssetData](TPair<FName, FAssetTagValueRef> Pair)
				{
					Class.InsertStatement.SetBindingValueByIndex(Class.TagBindingIndices[Pair.Key], Pair.Value.AsString());
				});

				if (Class.InsertStatement.Execute() == false)
				{
					UE_LOGF(LogAssetRegistryExport, Error, "Class Insert Error: %ls", *OutputDatabase.GetLastError());
				}
			}

			AssetsThisTransaction++;
			if (AssetsThisTransaction >= AssetsPerTransaction)
			{
				CommitTransactionStatement.GetPtrOrNull()->Execute();
				BeginTransactionStatement.GetPtrOrNull()->Execute();
				AssetsThisTransaction = 0;
			}
		}

		return true;
	});

	if (bCSV == false)
	{
		CommitTransactionStatement.GetPtrOrNull()->Execute();
	}
	else
	{
		FArchive* Ar = IFileManager::Get().CreateFileWriter(*(CSVFileName + TEXT("Assets.csv")));
		Ar->Serialize(AssetsCSV.GetData(), AssetsCSV.Num());
		delete Ar;

		for (TPair<FTopLevelAssetPath, FClassInfo>& Pair : ClassInfos)
		{
			FClassInfo& Class = Pair.Value;

			Ar = IFileManager::Get().CreateFileWriter(*(CSVFileName + TEXT("/Classes/") + Class.Name + TEXT("Assets.csv")));
			Ar->Serialize(Class.CSVMemory.GetData(), Class.CSVMemory.Num());
			delete Ar;
		}
	}

	double EndTime = FPlatformTime::Seconds();

	UE_LOGF(LogAssetRegistryExport, Display, "Done in %f seconds per 1000 assets, total of %f seconds.", (EndTime - StartTime) / (AssetCount / 1000.0), EndTime - StartTime);

	if (bGotCompressedSize == false)
	{
		UE_LOGF(LogAssetRegistryExport, Display, "Asset registry didn't have size metadata written back, CompressedSize DB column will be NULL.");
		UE_LOGF(LogAssetRegistryExport, Display, "Metadata can be written back via ProjectSettings/Packaging/WriteBackMetadataToAssetRegistry");
	}

	if (bCSV == false)
	{
		// Save a version so we can revisit if needed.
		OutputDatabase.SetUserVersion(1);
	
		// Statements have to be destroyed before the db can be closed.
		GlobalInsertStatement.GetPtrOrNull()->Destroy();
		CommitTransactionStatement.GetPtrOrNull()->Destroy();
		BeginTransactionStatement.GetPtrOrNull()->Destroy();
		for (TPair<FTopLevelAssetPath, FClassInfo>& Pair : ClassInfos)
		{
			Pair.Value.InsertStatement.Destroy();
		}

		if (OutputDatabase.Close() == false)
		{
			UE_LOGF(LogAssetRegistryExport, Error, "Close Error: %ls", *OutputDatabase.GetLastError());
		}
	}

	UE_LOGF(LogAssetRegistryExport, Display, "Done.");
	return 0;
}



class FAssetRegistryExportModule : public IModuleInterface
{
public:
	FAssetRegistryExportModule() { }
	virtual ~FAssetRegistryExportModule() {	}
	virtual void StartupModule() override {	}
};

IMPLEMENT_MODULE(FAssetRegistryExportModule, AssetRegistryExport);
