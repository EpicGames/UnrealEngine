// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Commandlets/Commandlet.h"
#include "AssetRegistryExportCommandlet.generated.h"

DECLARE_LOG_CATEGORY_EXTERN(LogAssetRegistryExport, Log, All);

UCLASS()
class UAssetRegistryExportCommandlet : public UCommandlet
{
	GENERATED_UCLASS_BODY()
public:
	virtual int32 Main(const FString& CmdLineParams) override;
};

